// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCache.h"
#include "Misc/ScopeLock.h"
#include "Stats/StatsMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "RuntimeAssetCachePluginInterface.h"
#include "RuntimeAssetCacheBackend.h"
#include "RuntimeAssetCacheBucket.h"
#include "Misc/RuntimeErrors.h"

DEFINE_STAT(STAT_RAC_ASyncWaitTime);

FRuntimeAssetCache::FRuntimeAssetCache()
{
	FString BucketName;
	int32 BucketSize;

	TArray<FString> BucketNames;
	GConfig->GetArray(TEXT("RuntimeAssetCache"), TEXT("BucketConfigs"), BucketNames, GEngineIni);

	for (FString& Bucket : BucketNames)
	{
		if (FParse::Value(*Bucket, TEXT("Name="), BucketName))
		{
			BucketSize = FRuntimeAssetCacheBucket::DefaultBucketSize;
			FParse::Value(*Bucket, TEXT("Size="), BucketSize);
			FRuntimeAssetCacheBucket* BucketDesc = FRuntimeAssetCacheBackend::Get().PreLoadBucket(FName(*BucketName), BucketSize);
			Buckets.Add(FName(*BucketName), BucketDesc);
		}
	}
}

FRuntimeAssetCache::~FRuntimeAssetCache()
{
	for (auto& Bucket : Buckets)
	{
		delete Bucket.Value;
	}

	Buckets.Empty();
}

int32 FRuntimeAssetCache::GetCacheSize(FName Bucket) const
{
	return ensureAsRuntimeWarning(Buckets.Contains(Bucket)) ? Buckets[Bucket]->GetSize() : 0;
}

int32 FRuntimeAssetCache::GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder, const FOnRuntimeAssetCacheAsyncComplete& OnComplete)
{
	if (ensureAsRuntimeWarning(CacheBuilder != nullptr))
	{
		int32 Handle = GetNextHandle();

		/** Must return a valid handle */
		check(Handle != 0);

		/** Make sure task isn't processed twice. */
		check(!PendingTasks.Contains(Handle));

		checkf(CacheBuilder->IsBuildThreadSafe(), TEXT("CacheBuilder %s Build function is not thread safe, but builder was used in asynchronous code. Use GetSynchronous instead."), CacheBuilder->GetBuilderName());

		FAsyncTask<FRuntimeAssetCacheAsyncWorker>* AsyncTask = new FAsyncTask<FRuntimeAssetCacheAsyncWorker>(CacheBuilder, &Buckets, Handle, OnComplete);

		{
			FScopeLock ScopeLock(&SynchronizationObject);
			PendingTasks.Add(Handle, AsyncTask);
		}
		AddToAsyncCompletionCounter(1);

		AsyncTask->StartBackgroundTask();

		return Handle;
	}
	else
	{
		// Using this with other functions is going to cause knock-on errors
		return 0;
	}
}

int32 FRuntimeAssetCache::GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder)
{
	return GetAsynchronous(CacheBuilder, FOnRuntimeAssetCacheAsyncComplete());
}

FVoidPtrParam FRuntimeAssetCache::GetSynchronous(IRuntimeAssetCacheBuilder* CacheBuilder)
{
	if (ensureAsRuntimeWarning(CacheBuilder))
	{
		if (ensureMsgf(!CacheBuilder->ShouldBuildAsynchronously(), TEXT("CacheBuilder %s can be only called asynchronously."), CacheBuilder->GetBuilderName()))
		{
			FAsyncTask<FRuntimeAssetCacheAsyncWorker>* AsyncTask = new FAsyncTask<FRuntimeAssetCacheAsyncWorker>(CacheBuilder, &Buckets, -1, FOnRuntimeAssetCacheAsyncComplete());
			AddToAsyncCompletionCounter(1);
			AsyncTask->StartSynchronousTask();
			return AsyncTask->GetTask().GetDataAndSize();
		}
	}
	return FVoidPtrParam::NullPtr();
}

bool FRuntimeAssetCache::ClearCache()
{
	/** Tell backend to clean itself up. */
	bool bResult = FRuntimeAssetCacheBackend::Get().ClearCache();

	if (!bResult)
	{
		return bResult;
	}

	/** If backend is cleaned up, clean up all buckets. */
	for (auto& Bucket : Buckets)
	{
		Bucket.Value->Reset();
	}

	return bResult;
}

bool FRuntimeAssetCache::ClearCache(FName BucketName)
{
	/** Tell backend to clean bucket up. */
	bool bResult = FRuntimeAssetCacheBackend::Get().ClearCache(BucketName);

	if (!bResult)
	{
		return bResult;
	}

	/** If backend is cleaned up, clean up all buckets. */
	for (auto& Bucket : Buckets)
	{
		Bucket.Value->Reset();
	}

	return bResult;
}

void FRuntimeAssetCache::AddToAsyncCompletionCounter(int32 Value)
{
	PendingTasksCounter.Add(Value);
	check(PendingTasksCounter.GetValue() >= 0);
}

void FRuntimeAssetCache::WaitAsynchronousCompletion(int32 Handle)
{
	STAT(double ThisTime = 0);
	{
		SCOPE_SECONDS_COUNTER(ThisTime);
		FAsyncTask<FRuntimeAssetCacheAsyncWorker>* AsyncTask = NULL;
		{
			FScopeLock ScopeLock(&SynchronizationObject);
			AsyncTask = PendingTasks.FindRef(Handle);
		}

		if (ensureAsRuntimeWarning(AsyncTask))
		{
			AsyncTask->EnsureCompletion();
		}
	}
	INC_FLOAT_STAT_BY(STAT_RAC_ASyncWaitTime, (float)ThisTime);
}

FVoidPtrParam FRuntimeAssetCache::GetAsynchronousResults(int32 Handle)
{
	FAsyncTask<FRuntimeAssetCacheAsyncWorker>* AsyncTask = nullptr;
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		PendingTasks.RemoveAndCopyValue(Handle, AsyncTask);
		if (ensureAsRuntimeWarning(AsyncTask))
		{
			AsyncTask->GetTask().FireCompletionDelegate();
		}
	}

	if (ensureAsRuntimeWarning(AsyncTask))
	{
		FVoidPtrParam Data = AsyncTask->GetTask().GetDataAndSize();
		delete AsyncTask;
		return Data;
	}
	else
	{
		return FVoidPtrParam::NullPtr();
	}
}

bool FRuntimeAssetCache::PollAsynchronousCompletion(int32 Handle)
{
	FAsyncTask<FRuntimeAssetCacheAsyncWorker>* AsyncTask = nullptr;
	{
		FScopeLock ScopeLock(&SynchronizationObject);
		AsyncTask = PendingTasks.FindRef(Handle);
	}
	ensureAsRuntimeWarning(AsyncTask);
	return AsyncTask ? AsyncTask->IsDone() : true;
}

void FRuntimeAssetCache::Tick()
{
	FScopeLock ScopeLock(&SynchronizationObject);
	for (auto Kvp : PendingTasks)
	{
		if (Kvp.Value->IsDone())
		{
			Kvp.Value->GetTask().FireCompletionDelegate();
		}
	}
}
