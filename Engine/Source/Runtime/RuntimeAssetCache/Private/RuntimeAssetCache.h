// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "HAL/ThreadSafeCounter.h"
#include "Stats/Stats.h"
#include "RuntimeAssetCacheInterface.h"
#include "RuntimeAssetCacheAsyncWorker.h"

class FRuntimeAssetCacheBucket;
class IRuntimeAssetCacheBuilder;

/** Stats. */
DECLARE_FLOAT_ACCUMULATOR_STAT_EXTERN(TEXT("RAC ASync Wait Time"), STAT_RAC_ASyncWaitTime, STATGROUP_RAC, );

/** Forward declarations. */
class IRuntimeAssetCacheBuilder;
class IRuntimeAssetCacheBackend;
class FCacheEntryMetadata;
class FRuntimeAssetCacheBucket;

/**
 * Concrete class implementing FRuntimeAssetCacheInterface. 
 */
class FRuntimeAssetCache final : public FRuntimeAssetCacheInterface
{
	/** FRuntimeAssetCacheInterface implementation */
public:
	virtual FVoidPtrParam GetSynchronous(IRuntimeAssetCacheBuilder* CacheBuilder) override;
	virtual int32 GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder, const FOnRuntimeAssetCacheAsyncComplete& OnCompletionDelegate) override;
	virtual int32 GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder) override;
	virtual int32 GetCacheSize(FName Bucket) const override;
	virtual bool ClearCache() override;
	virtual bool ClearCache(FName Bucket) override;
	virtual void WaitAsynchronousCompletion(int32 Handle) override;
	virtual FVoidPtrParam GetAsynchronousResults(int32 Handle) override;
	virtual bool PollAsynchronousCompletion(int32 Handle) override;
	virtual void AddToAsyncCompletionCounter(int32 Addend) override;
	virtual void Tick() override;
	/** End of FRuntimeAssetCacheInterface implementation */

public:
	/** Constructor */
	FRuntimeAssetCache();

	/** Destructor */
	~FRuntimeAssetCache();

private:

	/**
	* Generates next handle of async worker.
	* @return Worker handle.
	*/
	int32 GetNextHandle()
	{
		return CurrentAsyncTaskHandle.Increment();
	}

	/** Map of bucket names to their configs. */
	TMap<FName, FRuntimeAssetCacheBucket*> Buckets;

	/** Map of handle to pending task */
	TMap<int32, FAsyncTask<FRuntimeAssetCacheAsyncWorker>*> PendingTasks;

	/** Pending tasks synchronization object. */
	FCriticalSection SynchronizationObject;

	/** Counter used to generate worker handles. */
	FThreadSafeCounter CurrentAsyncTaskHandle;

	/** Number of pending tasks. */
	FThreadSafeCounter PendingTasksCounter;
};
