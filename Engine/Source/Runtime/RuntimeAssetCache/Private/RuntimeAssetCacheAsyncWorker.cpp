// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheAsyncWorker.h"
#include "RuntimeAssetCachePluginInterface.h"
#include "RuntimeAssetCacheBackend.h"
#include "RuntimeAssetCacheEntryMetadata.h"
#include "RuntimeAssetCacheBucketScopeLock.h"
#include "Misc/ScopeExit.h"
#include "RuntimeAssetCacheModule.h"

/** Stats */
DEFINE_STAT(STAT_RAC_NumBuilds);
DEFINE_STAT(STAT_RAC_NumCacheHits);
DEFINE_STAT(STAT_RAC_NumFails);
DEFINE_STAT(STAT_RAC_NumGets);
DEFINE_STAT(STAT_RAC_NumPuts);

FRuntimeAssetCacheAsyncWorker::FRuntimeAssetCacheAsyncWorker(IRuntimeAssetCacheBuilder* InCacheBuilder
	, TMap<FName, FRuntimeAssetCacheBucket*>* InBuckets
	, int32 InHandle
	, const FOnRuntimeAssetCacheAsyncComplete& InCompletionCallback)
	: CacheBuilder(InCacheBuilder)
	, Buckets(InBuckets)
	, CompletionCallback(InCompletionCallback)
	, Handle(InHandle)
	, bFiredCompletionDelegate(false)
{ }

void FRuntimeAssetCacheAsyncWorker::DoWork()
{
	ON_SCOPE_EXIT
	{
		/** Make sure completed work counter works properly regardless of where function is exited. */
		GetRuntimeAssetCache().AddToAsyncCompletionCounter(-1);
	};

	const TCHAR* Bucket = CacheBuilder->GetBucketConfigName();
	FName BucketName = FName(Bucket);
	FString CacheKey = BuildCacheKey(CacheBuilder);
	FName CacheKeyName = FName(*CacheKey);
	DataSize = 0;
	bEntryRetrieved = false;
	if (!Buckets->Contains(BucketName))
	{
		UE_LOGF(RuntimeAssetCache, Warning, "Caching asset %ls to unknown bucket %ls. Asset won't be cached.", *CacheBuilder->GetAssetUniqueName(), Bucket);
		Data = nullptr;
		return;
	}
	FRuntimeAssetCacheBucket* CurrentBucket = (*Buckets)[BucketName];
	INC_DWORD_STAT(STAT_RAC_NumGets);

	FCacheEntryMetadata* Metadata = nullptr;
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("RAC async get time"), STAT_RAC_AsyncGetTime, STATGROUP_RAC);
		Metadata = CurrentBucket->GetMetadata(CacheKey);
	}

	if (Metadata == nullptr)
	{
		FRuntimeAssetCacheBucketScopeLock Guard(*CurrentBucket);
		CurrentBucket->AddMetadataEntry(CacheKey, new FCacheEntryMetadata(FDateTime::MaxValue(), 0, 0, CacheKeyName), false);
	}
	else
	{
		if (Metadata->IsBuilding())
		{
			// Another worker is building this asset.
			Data = nullptr;
			return;
		}

		Metadata = FRuntimeAssetCacheBackend::Get().GetCachedData(BucketName, *CacheKey, Data, DataSize);
	}

	/* Entry found. */
	if (Metadata
		/* But was saved with older builder version. */
		&& !CacheBuilder->CachedAssetVersionIsUpToDate(Metadata->GetCachedAssetVersion()))
	{
		/* Pretend entry wasn't found, so it gets rebuilt. */
		FRuntimeAssetCacheBucketScopeLock Guard(*CurrentBucket);
		FRuntimeAssetCacheBackend::Get().RemoveCacheEntry(BucketName, *CacheKey);
		CurrentBucket->AddToCurrentSize(-Metadata->GetCachedAssetSize());
		CurrentBucket->RemoveMetadataEntry(CacheKey);
		delete Metadata;
		Metadata = nullptr;
	}

	if (Metadata)
	{
		INC_DWORD_STAT(STAT_RAC_NumCacheHits);
		FRuntimeAssetCacheBucketScopeLock Guard(*CurrentBucket);
		Metadata->SetLastAccessTime(FDateTime::Now());
		bEntryRetrieved = true;
		return;
	}

	{
		INC_DWORD_STAT(STAT_RAC_NumBuilds);
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("RAC async build time"), STAT_RAC_AsyncBuildTime, STATGROUP_RAC)
		FVoidPtrParam Results = CacheBuilder->Build();
		Data = Results.Data;
		DataSize = Results.DataSize;
	}

	if (!Data)
	{
		// Failed, cleanup data and return false.
		INC_DWORD_STAT(STAT_RAC_NumFails);
		DataSize = 0;
		bEntryRetrieved = false;
		CurrentBucket->RemoveMetadataEntry(CacheKey, true);
		return;
	}

	checkf(DataSize, TEXT("Size of asset to cache cannot be null. Asset cache key: %s"), *CacheKey);
	checkf(DataSize < CurrentBucket->GetSize(), TEXT("Cached asset is bigger than cache size. Increase cache size (%d) or reduce asset size(%lld). Asset cache key: %s"), CurrentBucket->GetSize(), DataSize, *CacheKey);

	FRuntimeAssetCacheBucketScopeLock Lock(*CurrentBucket);

	// Do we need to make some space in cache?
	int32 SizeOfSpaceToFree = CurrentBucket->GetCurrentSize() + DataSize - CurrentBucket->GetSize();
	if (SizeOfSpaceToFree > 0)
	{
		// Remove oldest entries from cache until we can fit upcoming entry.
		FreeCacheSpace(BucketName, SizeOfSpaceToFree);
	}

	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("RAC async put time"), STAT_RAC_PutTime, STATGROUP_RAC)
		FDateTime Now = FDateTime::Now();
		Metadata = CurrentBucket->GetMetadata(*CacheKey);
		if (Metadata)
		{
			Metadata->SetLastAccessTime(Now);
			if (Metadata->GetCachedAssetSize() == 0)
			{
				CurrentBucket->AddToCurrentSize(DataSize);
			}
			Metadata->SetCachedAssetSize(DataSize);
			Metadata->SetCachedAssetVersion(CacheBuilder->GetAssetVersion());
			CurrentBucket->AddMetadataEntry(CacheKey, Metadata, false);
		}
		else
		{
			Metadata = new FCacheEntryMetadata(Now, DataSize, CacheBuilder->GetAssetVersion(), CacheKeyName);
			CurrentBucket->AddMetadataEntry(CacheKey, Metadata, true);
		}
		FRuntimeAssetCacheBackend::Get().PutCachedData(BucketName, *CacheKey, Data, DataSize, Metadata);

		// Mark that building is finished AFTER putting data into
		// cache to avoid duplicate builds of the same entry.
		Metadata->FinishBuilding();
	}

	bEntryRetrieved = true;
}

TStatId FRuntimeAssetCacheAsyncWorker::GetStatId() const
{
	return TStatId();
}

void FRuntimeAssetCacheAsyncWorker::FireCompletionDelegate()
{
	check(IsInGameThread());
	if (!FiredCompletionDelegate())
	{
		bFiredCompletionDelegate = true;
		CompletionCallback.ExecuteIfBound(Handle, FVoidPtrParam(Data, DataSize));
	}
}

FString FRuntimeAssetCacheAsyncWorker::SanitizeCacheKey(const TCHAR* CacheKey)
{
	FString Output;
	FString Input(CacheKey);
	int32 StartValid = 0;
	int32 NumValid = 0;

	for (int32 i = 0; i < Input.Len(); i++)
	{
		if (FChar::IsAlnum(Input[i]) || FChar::IsUnderscore(Input[i]))
		{
			NumValid++;
		}
		else
		{
			// Copy the valid range so far
			Output += Input.Mid(StartValid, NumValid);

			// Reset valid ranges
			StartValid = i + 1;
			NumValid = 0;

			// Replace the invalid character with a special string
			Output += FString::Printf(TEXT("$%x"), uint32(Input[i]));
		}
	}

	// Just return the input if the entire string was valid
	if (StartValid == 0 && NumValid == Input.Len())
	{
		return Input;
	}
	else if (NumValid > 0)
	{
		// Copy the remaining valid part
		Output += Input.Mid(StartValid, NumValid);
	}

	return Output;
}

FString FRuntimeAssetCacheAsyncWorker::BuildCacheKey(const TCHAR* VersionString, const TCHAR* PluginSpecificCacheKeySuffix)
{
	FString UnsanitizedCacheKey = FString(VersionString);
	UnsanitizedCacheKey.AppendChars(PluginSpecificCacheKeySuffix, FCString::Strlen(PluginSpecificCacheKeySuffix));
	return SanitizeCacheKey(*UnsanitizedCacheKey);
}

FString FRuntimeAssetCacheAsyncWorker::BuildCacheKey(IRuntimeAssetCacheBuilder* CacheBuilder)
{
	return BuildCacheKey(CacheBuilder->GetBuilderName(), *CacheBuilder->GetAssetUniqueName());
}

void FRuntimeAssetCacheAsyncWorker::FreeCacheSpace(FName Bucket, int32 NumberOfBytesToFree)
{
	int32 AccumulatedSize = 0;
	FRuntimeAssetCacheBucket* CurrentBucket = (*Buckets)[Bucket];
	while (AccumulatedSize <= NumberOfBytesToFree)
	{
		FCacheEntryMetadata* OldestEntry = CurrentBucket->GetOldestEntry();
		checkf(!OldestEntry->IsBuilding(), TEXT("Cache is trying to remove asset before it finished building. Increase cache size. Asset name: %s, cache size: %d"), *OldestEntry->GetName().ToString(), CurrentBucket->GetSize());
		AccumulatedSize += OldestEntry->GetCachedAssetSize();
		FRuntimeAssetCacheBackend::Get().RemoveCacheEntry(Bucket, *OldestEntry->GetName().ToString());
		(*Buckets)[Bucket]->AddToCurrentSize(-OldestEntry->GetCachedAssetSize());
		CurrentBucket->RemoveMetadataEntry(OldestEntry->GetName().ToString());
	}
}

FArchive& operator<<(FArchive& Ar, FCacheEntryMetadata& Metadata)
{
	Ar << Metadata.CachedAssetSize;
	Ar << Metadata.CachedAssetVersion;
	FString String;
	if (Ar.IsLoading())
	{
		Ar << String;
		Metadata.Name = FName(*String);
	}
	else if (Ar.IsSaving())
	{
		Metadata.Name.ToString(String);
		Ar << String;
	}

	return Ar;
}
