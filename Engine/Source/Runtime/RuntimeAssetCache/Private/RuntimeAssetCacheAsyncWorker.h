// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "RuntimeAssetCacheInterface.h"
#include "RuntimeAssetCachePrivate.h"

class FRuntimeAssetCacheBucket;
class IRuntimeAssetCacheBuilder;
class UClass;

/** Stats */
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RAC Num Build"), STAT_RAC_NumBuilds, STATGROUP_RAC, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RAC Num Cache Hits"), STAT_RAC_NumCacheHits, STATGROUP_RAC, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RAC Num Retrieve fails"), STAT_RAC_NumFails, STATGROUP_RAC, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RAC Num Gets"), STAT_RAC_NumGets, STATGROUP_RAC, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("RAC Num Puts"), STAT_RAC_NumPuts, STATGROUP_RAC, );

/** Forward declarations. */
class IRuntimeAssetCacheBuilder;
class FRuntimeAssetCacheBucket;

/**
 * Worker to retrieve entry from cache or build it in case of cache miss.
 */
class FRuntimeAssetCacheAsyncWorker : public FNonAbandonableTask
{
public:
	/** Constructor */
	FRuntimeAssetCacheAsyncWorker(IRuntimeAssetCacheBuilder* InCacheBuilder, TMap<FName, FRuntimeAssetCacheBucket*>* InBuckets, int32 InHandle, const FOnRuntimeAssetCacheAsyncComplete& InCompletionCallback);

	/** Async task interface implementation. */
	void DoWork();
	TStatId GetStatId() const;
	/** End of async task interface implementation. */

	/** Gets serialized cache data. */
	void* GetData()
	{
		return Data;
	}

	/** Gets serialized cache data size in bytes. */
	int64 GetDataSize()
	{
		return DataSize;
	}

	/** Gets serialized cache data and data size. */
	FVoidPtrParam GetDataAndSize()
	{
		return FVoidPtrParam(Data, DataSize);
	}

	/**
	 * Fires completion delegate only if it wasn't fired earlier.
	 */
	void FireCompletionDelegate();

private:
	/**
	* Checks if task already fired completion delegate.
	* @return true if already fired completion delegate, false otherwise.
	*/
	bool FiredCompletionDelegate() const
	{
		return bFiredCompletionDelegate;
	}

	/**
	* Static function to make sure a cache key contains only legal characters by using an escape.
	* @param CacheKey Cache key to sanitize
	* @return Sanitized cache key
	**/
	static FString SanitizeCacheKey(const TCHAR* CacheKey);

	/**
	* Static function to build a cache key out of the plugin name, versions and plugin specific info
	* @param PluginName Name of the runtime asset data type
	* @param VersionString Version string of the runtime asset data
	* @param PluginSpecificCacheKeySuffix Whatever is needed to uniquely identify the specific cache entry.
	* @return Assembled cache key
	**/
	static FString BuildCacheKey(const TCHAR* VersionString, const TCHAR* PluginSpecificCacheKeySuffix);

	/**
	* Static function to build a cache key out of the CacheBuilder.
	* @param CacheBuilder Builder to create key from.
	* @return Assembled cache key
	**/
	static FString BuildCacheKey(IRuntimeAssetCacheBuilder* CacheBuilder);

	/**
	* Removes entries from cache until given number of bytes are freed (or more).
	* @param Bucket Bucket to clean up.
	* @param SizeOfSpaceToFreeInBytes Number of bytes to free (or more).
	*/
	void FreeCacheSpace(FName Bucket, int32 SizeOfSpaceToFreeInBytes);

	/** Cache builder to create cache entry in case of cache miss. */
	IRuntimeAssetCacheBuilder* CacheBuilder;
	
	/** Data to return to caller. */
	void* Data;

	/** Size of data returned to caller */
	int64 DataSize;

	/** Reference to map of bucket names to their descriptions. */
	TMap<FName, FRuntimeAssetCacheBucket*>* Buckets;

	/**
	 * True if successfully retrieved a cache entry. Can be false only when CacheBuilder
	 * returns false or no CacheBuilder was provided.
	 */
	bool bEntryRetrieved;

	/** Completion delegate called when cache entry is retrieved. */
	FOnRuntimeAssetCacheAsyncComplete CompletionCallback;

	/** Handle uniquely identifying this worker thread. */
	int32 Handle;

	/** True if completion delegate was already fired, false otherwise. */
	bool bFiredCompletionDelegate;

	UClass* Class;
};
