// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheBPHooks.h"
#include "RuntimeAssetCachePluginInterface.h"
#include "RuntimeAssetCacheModule.h"

FVoidPtrParam URuntimeAssetCacheBPHooks::GetSynchronous(TScriptInterface<IRuntimeAssetCacheBuilder> CacheBuilder)
{
	return GetRuntimeAssetCache().GetSynchronous(static_cast<IRuntimeAssetCacheBuilder*>(CacheBuilder.GetInterface()));
}

int32 URuntimeAssetCacheBPHooks::GetAsynchronous(TScriptInterface<IRuntimeAssetCacheBuilder> CacheBuilder, const FOnRuntimeAssetCacheAsyncComplete& CompletionDelegate)
{
	return GetRuntimeAssetCache().GetAsynchronous(static_cast<IRuntimeAssetCacheBuilder*>(CacheBuilder.GetInterface()), CompletionDelegate);
}

int32 URuntimeAssetCacheBPHooks::GetCacheSize(FName Bucket)
{
	return GetRuntimeAssetCache().GetCacheSize(Bucket);
}

bool URuntimeAssetCacheBPHooks::ClearCache(FName Bucket)
{
	return GetRuntimeAssetCache().ClearCache(Bucket);
}

void URuntimeAssetCacheBPHooks::WaitAsynchronousCompletion(int32 Handle)
{
	GetRuntimeAssetCache().WaitAsynchronousCompletion(Handle);
}

FVoidPtrParam URuntimeAssetCacheBPHooks::GetAsynchronousResults(int32 Handle)
{
	return GetRuntimeAssetCache().GetAsynchronousResults(Handle);
}

bool URuntimeAssetCacheBPHooks::PollAsynchronousCompletion(int32 Handle)
{
	return GetRuntimeAssetCache().PollAsynchronousCompletion(Handle);
}
