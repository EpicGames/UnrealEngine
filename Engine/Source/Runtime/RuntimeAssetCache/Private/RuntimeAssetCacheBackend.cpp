// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheBackend.h"
#include "RuntimeAssetCacheFilesystemBackend.h"
#include "RuntimeAssetCacheEntryMetadata.h"


FCacheEntryMetadata* FRuntimeAssetCacheBackend::GetCachedData(const FName Bucket, const TCHAR* CacheKey, void*& OutData, int64& OutDataSize)
{
	FCacheEntryMetadata* Result = nullptr;
	FArchive* Ar = CreateReadArchive(Bucket, CacheKey);
	if (!Ar)
	{
		return Result;
	}

	Result = PreloadMetadata(Ar);

	int64 TotalSize = Ar->TotalSize();
	int64 CurrentPosition = Ar->Tell();
	OutDataSize = TotalSize - CurrentPosition;
	OutData = new uint8[OutDataSize];
	Ar->Serialize(OutData, OutDataSize);
	Ar->Close();
	delete Ar;
	return Result;
}

bool FRuntimeAssetCacheBackend::PutCachedData(const FName Bucket, const TCHAR* CacheKey, void* InData, int64 InDataSize, FCacheEntryMetadata* Metadata)
{
	bool bResult = false;
	FArchive* Ar = CreateWriteArchive(Bucket, CacheKey);
	if (!Ar)
	{
		return bResult;
	}

	*Ar << *Metadata;
	Ar->Serialize(InData, InDataSize);
	bResult = Ar->Close();
	delete Ar;
	return bResult;
}

FRuntimeAssetCacheBackend* FRuntimeAssetCacheBackend::CreateBackend()
{
	return new FRuntimeAssetCacheFilesystemBackend();
}

FCacheEntryMetadata* FRuntimeAssetCacheBackend::PreloadMetadata(FArchive* Ar)
{
	FCacheEntryMetadata* Result = new FCacheEntryMetadata();
	*Ar << *Result;
	return Result;
}
