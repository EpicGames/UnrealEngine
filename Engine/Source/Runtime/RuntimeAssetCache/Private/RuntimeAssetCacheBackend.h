// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FArchive;
class FCacheEntryMetadata;
class FRuntimeAssetCacheBucket;

/** Forward declarations. */
class FCacheEntryMetadata;
class FRuntimeAssetCacheBucket;

/**
 * Facade for runtime asset cache backend. Currently only file system backend is available.
 */
class FRuntimeAssetCacheBackend : public FNoncopyable
{
public:
	/**
	 * Singleton getter.
	 * @return Backend implementation.
	 */
	static FRuntimeAssetCacheBackend& Get()
	{
		static FRuntimeAssetCacheBackend& Backend = *CreateBackend();
		return Backend;
	}

	/**
	 * Retrieves cached entry.
	 * @param Bucket Bucket to query.
	 * @param CacheKey Key of cache entry to retrieve.
	 * @param OutData Serialized cache entry. Emptied if entry not found.
	 * @return Metadata descriptor of cache entry on cache hit, nullptr otherwise.
	 */
	FCacheEntryMetadata* GetCachedData(FName Bucket, const TCHAR* CacheKey, void*& OutData, int64& OutDataSize);

	/**
	* Puts entry to cache.
	* @param Bucket Bucket to put entry to.
	* @param CacheKey Key of cache entry to retrieve.
	* @param InData Serialized cache entry.
	* @param Metadata descriptor of cache entry.
	* @return True if successfully put entry into cache.
	*/
	bool PutCachedData(FName Bucket, const TCHAR* CacheKey, void* InData, int64 InDataSize, FCacheEntryMetadata* Metadata);

	/**
	* Removes entry from cache.
	* @param Bucket Bucket to remove entry from.
	* @param CacheKey Key of cache entry to remove.
	* @return True if successfully removed entry from cache.
	*/
	virtual bool RemoveCacheEntry(FName Bucket, const TCHAR* CacheKey) = 0;

	/**
	* Removes all entries from cache.
	* @return True if successfully removed all entries from cache.
	*/
	virtual bool ClearCache() = 0;

	/**
	* Removes all entries from given bucket.
	* @param Bucket Bucket to clean.
	* @return True if successfully removed all entries from cache.
	*/
	virtual bool ClearCache(FName Bucket) = 0;

	/**
	* Preloads cache metadata and size for given bucket.
	* @param BucketName Name of bucket to preload.
	* @param BucketSize Size of bucket to preload.
	* @return Preloaded bucket on success, nullptr otherwise.
	*/
	virtual FRuntimeAssetCacheBucket* PreLoadBucket(FName BucketName, int32 BucketSize) = 0;

protected:
	/*
	* Virtual destructor.
	*/
	virtual ~FRuntimeAssetCacheBackend() { }

	/*
	* Creates archive to read data from.
	* @param Bucket Bucket containing cache entry.
	* @param CacheKey Key to cache entry.
	* @return Archive storing data.
	*/
	virtual FArchive* CreateReadArchive(FName Bucket, const TCHAR* CacheKey) = 0;

	/*
	* Creates archive to write data into.
	* @param Bucket Bucket containing cache entry.
	* @param CacheKey Key to cache entry.
	* @return Archive storing data.
	*/
	virtual FArchive* CreateWriteArchive(FName Bucket, const TCHAR* CacheKey) = 0;

	/*
	* Preloads metadata from archive.
	* @param Ar Archive to load metadata from
	* @return Metadata.
	*/
	FCacheEntryMetadata* PreloadMetadata(FArchive* Ar);

private:
	/**
	 * Factory method to create actual backend implementation.
	 * @return Cache backend.
	 */
	static FRuntimeAssetCacheBackend* CreateBackend();

};
