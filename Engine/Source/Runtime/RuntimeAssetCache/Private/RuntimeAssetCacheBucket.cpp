// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheBucket.h"
#include "Misc/DateTime.h"
#include "RuntimeAssetCacheEntryMetadata.h"

const int32 FRuntimeAssetCacheBucket::DefaultBucketSize = 5 * 1024 * 1024;

FRuntimeAssetCacheBucket::~FRuntimeAssetCacheBucket()
{
	for (auto& Entry : CacheMetadata)
	{
		delete Entry.Value;
	}
}

void FRuntimeAssetCacheBucket::Reset()
{
	FScopeLock Lock(&MetadataCriticalSection);
	for (auto& Kvp : CacheMetadata)
	{
		delete Kvp.Value;
		Kvp.Value = nullptr;
	}
	CacheMetadata.Empty();
	CurrentSize = 0;
}

FCacheEntryMetadata* FRuntimeAssetCacheBucket::GetMetadata(const FString& Key)
{
	FScopeLock Lock(&MetadataCriticalSection);
	return CacheMetadata.FindOrAdd(Key);
}

void FRuntimeAssetCacheBucket::RemoveMetadataEntry(const FString& Key, bool bBuildFailed)
{
	FScopeLock Lock(&MetadataCriticalSection);
#if UE_BUILD_DEBUG
	FCacheEntryMetadata* Entry = CacheMetadata.FindOrAdd(Key);
	if (Entry && !bBuildFailed)
	{
		check(!Entry->IsBuilding());
	}
#endif
	CacheMetadata.Remove(Key);
}

void FRuntimeAssetCacheBucket::AddMetadataEntry(const FString& Key, FCacheEntryMetadata* Value, bool bUpdateSize)
{
	check(Value);
	FScopeLock LockMetadata(&MetadataCriticalSection);
	FCacheEntryMetadata*& OldValue = CacheMetadata.FindOrAdd(Key);
	if (bUpdateSize)
	{
		FScopeLock LockCurrentSize(&CurrentSizeCriticalSection);
		if (OldValue)
		{
			/** If overwriting, keep proper record of used cache size. */
			AddToCurrentSize(-OldValue->GetCachedAssetSize());
		}
		AddToCurrentSize(Value->GetCachedAssetSize());
	}
	OldValue = Value;
}

FCacheEntryMetadata* FRuntimeAssetCacheBucket::GetOldestEntry()
{
	FScopeLock Lock(&MetadataCriticalSection);
	FDateTime OldestValue = FDateTime::MaxValue();
	FCacheEntryMetadata* Result = nullptr;
	for (auto& Kvp : CacheMetadata)
	{
		if (Kvp.Value && Kvp.Value->GetLastAccessTime() < OldestValue)
		{
			OldestValue = Kvp.Value->GetLastAccessTime();
			Result = Kvp.Value;
		}
	}

	return Result;
}
