// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Misc/ScopeLock.h"

class FCacheEntryMetadata;

/** Forward declarations. */
class FCacheEntryMetadata;

/**
 * Holds data about cache entries.
 */
class FRuntimeAssetCacheBucket
{
public:
	/** Default constructor. */
	FRuntimeAssetCacheBucket()
		: Size(DefaultBucketSize)
		, CurrentSize(0)
	{ }

	/**
	 * Constructor
	 * @param InSize Size of bucket in bytes.
	 */
	FRuntimeAssetCacheBucket(int32 InSize)
		: Size(InSize)
		, CurrentSize(0)
	{ }

	/** Destructor. */
	~FRuntimeAssetCacheBucket();

	/** Gets size in bytes of the bucket. */
	int32 GetSize() const
	{
		return Size;
	}

	/**
	 * Cleans up all metadata entries.
	 */
	void Reset();

	/**
	 * Gets metadata entry for given key.
	 * @param Key Key to get metadata for.
	 * @return Metadata for given key. Default value if Key is not found.
	 */
	FCacheEntryMetadata* GetMetadata(const FString& Key);

	/**
	* Removes metadata entry for given key.
	* @param Key Key to remove metadata for.
	* @param bBuildFailed Indicates that building cache entry failed and some checks should be skipped.
	*/
	void RemoveMetadataEntry(const FString& Key, bool bBuildFailed = false);

	/**
	* Adds metadata entry for given key.
	* @param Key Key to add metadata for.
	* @param Value Metadata to add. If there is an entry for given key, it gets overwritten.
	* @param bUpdateSize If true, function will update current size of bucket. Otherwise, it's up to caller.
	*/
	void AddMetadataEntry(const FString& Key, FCacheEntryMetadata* Value, bool bUpdateSize);

	/**
	* Gets number of bytes used in cache so far.
	*/
	int32 GetCurrentSize()
	{
		FScopeLock Lock(&CurrentSizeCriticalSection);
		return CurrentSize;
	}

	/**
	* Adds to number of bytes used in cache so far.
	* @param Value Number of bytes to add. Can be negative.
	*/
	void AddToCurrentSize(int32 Value)
	{
		FScopeLock Lock(&CurrentSizeCriticalSection);
		CurrentSize += Value;
		check(CurrentSize <= Size);
	}

	/**
	* Gets oldest metadata entry in cache.
	* @return Oldest metadata entry.
	*/
	FCacheEntryMetadata* GetOldestEntry();

	/** Default size of bucket in case it wasn't specified in config. */
	static const int32 DefaultBucketSize;

private:
	/** Map of cache keys to cache entries metadata. */
	TMap<FString, FCacheEntryMetadata*> CacheMetadata;

	/** Guard for modifying CacheMetadata (adding/removing elements). Each element has individual guard for modifying it. */
	FCriticalSection MetadataCriticalSection;

	/** Cache bucket size in bytes. */
	int32 Size;

	/** Used bucket size in bytes. */
	int32 CurrentSize;

	/** Guard for modifying current size from multiple threads. */
	FCriticalSection CurrentSizeCriticalSection;

	/** Helper friend class to guard critical sections on scope basis. */
	friend class FRuntimeAssetCacheBucketScopeLock;
};
