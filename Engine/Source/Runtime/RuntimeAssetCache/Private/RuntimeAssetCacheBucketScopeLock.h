// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RuntimeAssetCacheBucket.h"

/*
 * Scope lock guarding RuntimeAssetCache bucket metadata critical section.
 **/
class FRuntimeAssetCacheBucketScopeLock
{
public:
	FRuntimeAssetCacheBucketScopeLock(FRuntimeAssetCacheBucket& InBucket)
		: Bucket(InBucket)
	{
		Bucket.MetadataCriticalSection.Lock();
	}

	~FRuntimeAssetCacheBucketScopeLock()
	{
		Bucket.MetadataCriticalSection.Unlock();
	}

private:
	/** Bucket to protect. */
	FRuntimeAssetCacheBucket& Bucket;
};
