// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheBuilders.h"
#include "Serialization/LargeMemoryWriter.h"
#include "RuntimeAssetCacheModule.h"
#include "TextureResource.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

void URuntimeAssetCacheBuilder_ObjectBase::SaveNewAssetToCache(UObject* NewAsset)
{
	SetAsset(NewAsset);
	GetFromCacheAsync(OnAssetCacheComplete);
}

void URuntimeAssetCacheBuilder_ObjectBase::SetAsset(UObject* NewAsset)
{
	Asset = NewAsset;
	OnSetAsset(Asset);
}

FVoidPtrParam URuntimeAssetCacheBuilder_ObjectBase::Build()
{
	// There was no cached asset, so this is expecting us to return the data that needs to be saved to disk
	// If we have no asset created yet, just return null. That will trigger the async creation of the asset.
	// If we do have an asset, serialize it here into a good format and return a pointer to that memory buffer.
	if (Asset)
	{
		FLargeMemoryWriter Ar(GetSerializedDataSizeEstimate(), /*bIsPersistent*/ true);
		SerializeAsset(Ar);

		FVoidPtrParam Result(Ar.GetData(), Ar.TotalSize());
		Ar.ReleaseOwnership();
		return Result;
	}

	// null
	return FVoidPtrParam::NullPtr();
}

void URuntimeAssetCacheBuilder_ObjectBase::GetFromCacheAsync(const FOnAssetCacheComplete& OnComplete)
{
	OnAssetCacheComplete = OnComplete;
	GetFromCacheAsyncCompleteDelegate.BindDynamic(this, &URuntimeAssetCacheBuilder_ObjectBase::GetFromCacheAsyncComplete);
	CacheHandle = GetRuntimeAssetCache().GetAsynchronous(this, GetFromCacheAsyncCompleteDelegate);
}

void URuntimeAssetCacheBuilder_ObjectBase::GetFromCacheAsyncComplete(int32 Handle, FVoidPtrParam DataPtr)
{
	if (Handle != CacheHandle)
	{
		// This can sometimes happen when the world changes and everything couldn't cancel correctly. Just ignore any callbacks that don't match handles.
		if (DataPtr.Data != nullptr)
		{
			FMemory::Free(DataPtr.Data);
		}
		return;
	}

	if (DataPtr.Data != nullptr)
	{
		// Success! Finished loading or saving data from cache
		// If saving, then we already have the right data and we can just report success
		if (Asset == nullptr)
		{
			// If loading, we now need to serialize the data into a usable format

			// Make sure Asset is set up to be loaded into
			OnAssetPreLoad();

			FBufferReader Ar(DataPtr.Data, DataPtr.DataSize, false);
			SerializeAsset(Ar);

			// Perform any specific init functions after load
			OnAssetPostLoad();
		}

		// Free the buffer memory on both save and load
		// On save the buffer gets created in Build()
		// On load the buffer gets created in FRuntimeAssetCacheBackend::GetCachedData()
		FMemory::Free(DataPtr.Data);
		CacheHandle = 0;

		// Success!
		OnAssetCacheComplete.ExecuteIfBound(this, true);
	}
	else
	{
		// Data not on disk. Kick off the creation process.
		// Once complete, call GetFromCacheAsync() again and it will loop back to this function, but should succeed.
		if (!bProcessedCacheMiss)
		{
			bProcessedCacheMiss = -1;
			OnAssetCacheMiss();
		}
		else
		{
			// Failed
			OnAssetCacheComplete.ExecuteIfBound(this, false);
		}
	}
}

void UExampleTextureCacheBuilder::OnSetAsset(UObject* NewAsset)
{
	Texture = Cast<UTexture2D>(NewAsset);
}

void UExampleTextureCacheBuilder::OnAssetCacheMiss_Implementation()
{
	// Override and create the new asset here (this is where we would render to a render target, then get the result)
	// For this example we will simply load an existing texture
	UTexture2D* NewTexture = LoadObject<UTexture2D>(nullptr, *AssetName);

	// Make sure the new asset gets properly cached for next time.
	SaveNewAssetToCache(NewTexture);
}

void UExampleTextureCacheBuilder::SerializeAsset(FArchive& Ar)
{
	if (Texture && Texture->GetPlatformData())
	{
		FTexturePlatformData* PlatformData = Texture->GetPlatformData();
		UEnum* PixelFormatEnum = UTexture::GetPixelFormatEnum();

		Ar << PlatformData->SizeX;
		Ar << PlatformData->SizeY;
		Ar << PlatformData->PackedData;
		if (Ar.IsLoading())
		{
			FString PixelFormatString;
			Ar << PixelFormatString;
			PlatformData->PixelFormat = (EPixelFormat)PixelFormatEnum->GetValueByName(*PixelFormatString);
		}
		else if (Ar.IsSaving())
		{
			FString PixelFormatString = PixelFormatEnum->GetNameByValue(PlatformData->PixelFormat).GetPlainNameString();
			Ar << PixelFormatString;
		}

		if (PlatformData->GetHasOptData())
		{
			Ar << PlatformData->OptData;
		}

		int32 NumMips = PlatformData->Mips.Num();
		int32 FirstMip = 0;
		int32 LastMip = NumMips;

		TArray<uint32> SavedFlags;
		if (Ar.IsSaving())
		{
			// Force resident mips inline
			SavedFlags.Empty(NumMips);
			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				SavedFlags.Add(PlatformData->Mips[MipIndex].BulkData.GetBulkDataFlags());
				PlatformData->Mips[MipIndex].BulkData.SetBulkDataFlags(BULKDATA_ForceInlinePayload | BULKDATA_SingleUse);
			}

			// Don't save empty Mips
			while (FirstMip < NumMips && PlatformData->Mips[FirstMip].BulkData.GetBulkDataSize() <= 0)
			{
				FirstMip++;
			}
			for (int32 MipIndex = FirstMip + 1; MipIndex < NumMips; ++MipIndex)
			{
				if (PlatformData->Mips[FirstMip].BulkData.GetBulkDataSize() <= 0)
				{
					// This means there are empty tail mips, which should never happen
					// If it does, simply don't save any mips after this point.
					LastMip = MipIndex;
					break;
				}
			}
			int32 NumMipsSaved = LastMip - FirstMip;
			Ar << NumMipsSaved;
		}

		if (Ar.IsLoading())
		{
			Ar << NumMips;
			LastMip = NumMips;
			PlatformData->Mips.Empty(NumMips);
			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				PlatformData->Mips.Add(new FTexture2DMipMap(0, 0));
			}
		}

		EBulkDataLockFlags LockFlags = Ar.IsSaving() ? LOCK_READ_ONLY : LOCK_READ_WRITE;
		for (int32 MipIndex = FirstMip; MipIndex < LastMip; ++MipIndex)
		{
			FTexture2DMipMap& Mip = PlatformData->Mips[MipIndex];
			int32 SizeX = Mip.SizeX;
			int32 SizeY = Mip.SizeY;
			Ar << SizeX;
			Ar << SizeY;
			Mip.SizeX = SizeX;
			Mip.SizeY = SizeY;

			int32 BulkDataSizeInBytes = Mip.BulkData.GetBulkDataSize();
			Ar << BulkDataSizeInBytes;
			if (BulkDataSizeInBytes > 0)
			{
				void* BulkMipData = Mip.BulkData.Lock(LockFlags);
				if (Ar.IsLoading())
				{
					int32 ElementCount = BulkDataSizeInBytes / Mip.BulkData.GetElementSize();
					BulkMipData = Mip.BulkData.Realloc(ElementCount);
				}
				Ar.Serialize(BulkMipData, BulkDataSizeInBytes);
				Mip.BulkData.Unlock();
			}
		}

		// Restore flags
		if (Ar.IsSaving())
		{
			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				PlatformData->Mips[MipIndex].BulkData.SetBulkDataFlags(SavedFlags[MipIndex]);
			}
		}
	}
}

void UExampleTextureCacheBuilder::OnAssetPreLoad()
{
	// Create an object to load the data into
	UTexture2D* NewTexture = NewObject<UTexture2D>();
	NewTexture->SetPlatformData(new FTexturePlatformData());
	NewTexture->NeverStream = true;

	SetAsset(NewTexture);
}

void UExampleTextureCacheBuilder::OnAssetPostLoad()
{
	Texture->UpdateResource();
}

int64 UExampleTextureCacheBuilder::GetSerializedDataSizeEstimate()
{
	int64 DataSize = sizeof(FTexturePlatformData);
	DataSize += sizeof(FString) + (sizeof(TCHAR) * 12);								// Guess the size of the pixel format string (most are less than 12 characters, but we don't need to be exact)
	DataSize += Texture->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);	// Size of all the mips
	DataSize += (sizeof(int32) * 3) * Texture->GetNumMips();						// Each mip stores its X and Y size, and its BulkDataSize
	return DataSize;
}
