// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DateTime.h"
#include "UObject/NameTypes.h"
#include "HAL/ThreadSafeBool.h"

class FArchive;

/**
 * Cache entry metadata.
 */
class FCacheEntryMetadata
{
public:
	FCacheEntryMetadata(const FDateTime& InLastAccessTime
		, int32 InCachedAssetSize
		, int32 InCachedAssetVersion
		, FName InName)
		: LastAccessTime(InLastAccessTime)
		, CachedAssetSize(InCachedAssetSize)
		, CachedAssetVersion(InCachedAssetVersion)
		, Name(InName)
		, bIsBuilding(true)
	{ }

	FCacheEntryMetadata()
		: LastAccessTime(FDateTime::Now())
		, CachedAssetSize(0)
		, CachedAssetVersion(0)
		, Name(NAME_None)
		, bIsBuilding(false)
	{ }

	int32 GetCachedAssetVersion() const
	{
		return CachedAssetVersion;
	}

	void SetCachedAssetVersion(int32 Version)
	{
		CachedAssetVersion = Version;
	}

	int32 GetCachedAssetSize() const
	{
		return CachedAssetSize;
	}

	void SetCachedAssetSize(int64 Value)
	{
		CachedAssetSize = Value;
	}

	void SetLastAccessTime(FDateTime Value)
	{
		LastAccessTime = Value;
	}

	FDateTime GetLastAccessTime() const
	{
		return LastAccessTime;
	}

	FName GetName() const
	{
		return Name;
	}

	bool IsBuilding() const
	{
		return bIsBuilding;
	}

	void FinishBuilding()
	{
		bIsBuilding = false;
	}

	friend FArchive& operator<<(FArchive& Ar, FCacheEntryMetadata& Metadata);

private:
	FDateTime LastAccessTime;
	int64 CachedAssetSize;
	int32 CachedAssetVersion;
	FName Name;
	FThreadSafeBool bIsBuilding;
};

FArchive& operator<<(FArchive& Ar, FCacheEntryMetadata& Metadata);
