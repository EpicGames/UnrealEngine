// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheFilesystemBackend.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "RuntimeAssetCacheBucket.h"
#include "RuntimeAssetCacheEntryMetadata.h"

FArchive* FRuntimeAssetCacheFilesystemBackend::CreateReadArchive(FName Bucket, const TCHAR* CacheKey)
{
	FString Path = FPaths::Combine(*PathToRAC, *Bucket.ToString(), CacheKey);
	return IFileManager::Get().CreateFileReader(*Path);
}

FArchive* FRuntimeAssetCacheFilesystemBackend::CreateWriteArchive(FName Bucket, const TCHAR* CacheKey)
{
	FString Path = FPaths::Combine(*PathToRAC, *Bucket.ToString(), CacheKey);
	return IFileManager::Get().CreateFileWriter(*Path);
}

FRuntimeAssetCacheFilesystemBackend::FRuntimeAssetCacheFilesystemBackend()
{
	GConfig->GetString(TEXT("RuntimeAssetCache"), TEXT("PathToRAC"), PathToRAC, GEngineIni);
	PathToRAC = FPaths::ProjectSavedDir() / PathToRAC;
}

bool FRuntimeAssetCacheFilesystemBackend::RemoveCacheEntry(const FName Bucket, const TCHAR* CacheKey)
{
	FString Path = FPaths::Combine(*PathToRAC, *Bucket.ToString(), CacheKey);
	return IFileManager::Get().Delete(*Path);
}

bool FRuntimeAssetCacheFilesystemBackend::ClearCache()
{
	return IFileManager::Get().DeleteDirectory(*PathToRAC, false, true);
}

bool FRuntimeAssetCacheFilesystemBackend::ClearCache(FName Bucket)
{
	if (Bucket != NAME_None)
	{
		return IFileManager::Get().DeleteDirectory(*FPaths::Combine(*PathToRAC, *Bucket.ToString()), false, true);
	}

	return false;
}

FRuntimeAssetCacheBucket* FRuntimeAssetCacheFilesystemBackend::PreLoadBucket(FName BucketName, int32 BucketSize)
{
	FString Path = FPaths::Combine(*PathToRAC, *BucketName.ToString());
	FRuntimeAssetCacheBucket* Result = new FRuntimeAssetCacheBucket(BucketSize);

	class FRuntimeAssetCacheFilesystemBackendDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FRuntimeAssetCacheFilesystemBackendDirectoryVisitor(FRuntimeAssetCacheBucket* InBucket, FName InBucketName, FRuntimeAssetCacheFilesystemBackend* InBackend)
			: Bucket(InBucket)
			, BucketName(InBucketName)
			, Backend(InBackend)
		{ }

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory)
			{
				return true;
			}
			FString CacheKey = FPaths::GetBaseFilename(FilenameOrDirectory);
			FArchive* Ar = Backend->CreateReadArchive(BucketName, *CacheKey);
			FCacheEntryMetadata* Metadata = Backend->PreloadMetadata(Ar);
			Files.Add(TFilenameMetadataPair(*FPaths::GetBaseFilename(FilenameOrDirectory), Metadata));
			delete Ar;
			return true;
		}

		void Finalize()
		{
			Files.Sort([](const TFilenameMetadataPair& A, const TFilenameMetadataPair& B) { return A.Get<1>()->GetLastAccessTime() > B.Get<1>()->GetLastAccessTime(); });

			int32 CurrentFileIndex = 0;
			for (; CurrentFileIndex < Files.Num(); ++CurrentFileIndex)
			{
				FName Name = Files[CurrentFileIndex].Get<0>();
				FCacheEntryMetadata* Metadata = Files[CurrentFileIndex].Get<1>();

				if ((Bucket->GetCurrentSize() + Metadata->GetCachedAssetSize()) <= Bucket->GetSize())
				{
					Bucket->AddMetadataEntry(Name.ToString(), Metadata, true);
				}
				else
				{
					// No space left. Stop adding entries to the bucket, and all the rest of the older files
					break;
				}
			}

			for (; CurrentFileIndex < Files.Num(); ++CurrentFileIndex)
			{
				FName Name = Files[CurrentFileIndex].Get<0>();
				FCacheEntryMetadata* Metadata = Files[CurrentFileIndex].Get<1>();
				Backend->RemoveCacheEntry(BucketName, *Name.ToString());
				delete Metadata;
			}
		}

	private:
		FRuntimeAssetCacheBucket* Bucket;
		FName BucketName;
		FRuntimeAssetCacheFilesystemBackend* Backend;

		typedef TTuple<FName, FCacheEntryMetadata*> TFilenameMetadataPair;
		TArray<TFilenameMetadataPair> Files;
	} Visitor(Result, BucketName, this);
	IFileManager::Get().IterateDirectory(*Path, Visitor);
	Visitor.Finalize();

	return Result;
}
