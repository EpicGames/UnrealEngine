// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "RuntimeAssetCacheBackend.h"

class FArchive;
class FRuntimeAssetCacheBucket;

/**
 * File system implementation of runtime asset cache backend.
 */
class FRuntimeAssetCacheFilesystemBackend : public FRuntimeAssetCacheBackend
{
public:
	FRuntimeAssetCacheFilesystemBackend();

	/** FRuntimeAssetCacheBackend interface implementation. */
	virtual bool RemoveCacheEntry(FName Bucket, const TCHAR* CacheKey) override;
	virtual bool ClearCache() override;
	virtual bool ClearCache(FName Bucket) override;
	virtual FRuntimeAssetCacheBucket* PreLoadBucket(FName BucketName, int32 BucketSize) override;
protected:
	virtual FArchive* CreateReadArchive(FName Bucket, const TCHAR* CacheKey) override;
	virtual FArchive* CreateWriteArchive(FName Bucket, const TCHAR* CacheKey) override;
	/** End of FRuntimeAssetCacheBackend interface implementation. */

private:

	/** File system path to runtime asset cache. */
	FString PathToRAC;
};
