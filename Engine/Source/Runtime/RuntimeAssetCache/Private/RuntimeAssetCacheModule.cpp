// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeAssetCacheModule.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "RuntimeAssetCache.h"
#include "Engine/World.h"

DEFINE_LOG_CATEGORY(RuntimeAssetCache)

FRuntimeAssetCacheInterface& GetRuntimeAssetCache()
{
	static FRuntimeAssetCacheModuleInterface& Module = FModuleManager::LoadModuleChecked<FRuntimeAssetCacheModuleInterface>("RuntimeAssetCache");
	static FRuntimeAssetCacheInterface& Interface = Module.GetRuntimeAssetCache();
	return Interface;
}

/**
 * Class implementing RuntimeAssetCache module interface. This is split
 * to solve circular dependency when building UHT.
 */
class FRuntimeAssetCacheModule : public FRuntimeAssetCacheModuleInterface
{
public:
	/**
	* Gets runtime asset cache.
	* @return Runtime asset cache.
	*/
	virtual FRuntimeAssetCacheInterface& GetRuntimeAssetCache() override
	{
		static FRuntimeAssetCache StaticRuntimeAssetCache;
		return StaticRuntimeAssetCache;
	}

	void StartupModule() override
	{
		FWorldDelegates::OnWorldTickStart.AddStatic(&FRuntimeAssetCacheModule::TickRuntimeAssetCache);
	}

	static void TickRuntimeAssetCache(UWorld* World, ELevelTick TickType, float DeltaSeconds)
	{
		static FRuntimeAssetCacheModuleInterface& Module = FModuleManager::LoadModuleChecked<FRuntimeAssetCacheModuleInterface>("RuntimeAssetCache");
		static FRuntimeAssetCacheInterface& Interface = Module.GetRuntimeAssetCache();
		Interface.Tick();
	}
};

IMPLEMENT_MODULE(FRuntimeAssetCacheModule, RuntimeAssetCache);
