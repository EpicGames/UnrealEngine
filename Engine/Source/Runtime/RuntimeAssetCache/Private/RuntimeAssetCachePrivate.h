// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Stats/Stats.h"

/** Stats */
DECLARE_STATS_GROUP(TEXT("RAC"), STATGROUP_RAC, STATCAT_Advanced);

/** Logs */
DECLARE_LOG_CATEGORY_EXTERN(RuntimeAssetCache, Log, All);
