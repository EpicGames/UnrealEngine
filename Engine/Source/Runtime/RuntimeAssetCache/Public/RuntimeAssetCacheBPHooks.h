// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RuntimeAssetCacheInterface.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectGlobals.h"

#include "RuntimeAssetCacheBPHooks.generated.h"

/** Forward declarations */
class FOnRuntimeAssetCacheAsyncComplete;
class IRuntimeAssetCacheBuilder;
class UObject;
struct FFrame;

UCLASS()
class URuntimeAssetCacheBPHooks : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static FVoidPtrParam GetSynchronous(TScriptInterface<IRuntimeAssetCacheBuilder> CacheBuilder);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static int32 GetAsynchronous(TScriptInterface<IRuntimeAssetCacheBuilder> CacheBuilder, const FOnRuntimeAssetCacheAsyncComplete& CompletionDelegate);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static int32 GetCacheSize(FName Bucket);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static bool ClearCache(FName Bucket);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static void WaitAsynchronousCompletion(int32 Handle);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static FVoidPtrParam GetAsynchronousResults(int32 Handle);

	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	static bool PollAsynchronousCompletion(int32 Handle);
};
