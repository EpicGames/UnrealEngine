// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "UObject/ScriptMacros.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Stack.h"
#include "RuntimeAssetCacheInterface.h"
#include "RuntimeAssetCachePluginInterface.h"
#include "RuntimeAssetCacheBuilders.generated.h"

class FArchive;
class UTexture2D;

DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnAssetCacheComplete, URuntimeAssetCacheBuilder_ObjectBase*, CachedAssetBuilder, bool, Success);

class UTexture2D;

UCLASS(BlueprintType)
class RUNTIMEASSETCACHE_API URuntimeAssetCacheBuilder_ObjectBase : public UObject, public IRuntimeAssetCacheBuilder
{
	GENERATED_BODY()
public:
	virtual void PostInitProperties() override
	{
		Super::PostInitProperties();
		CacheHandle = 0;
		bProcessedCacheMiss = false;
	}

	virtual const TCHAR* GetBucketConfigName() const override
	{
		return TEXT("DefaultBucket");
	}

	virtual const TCHAR* GetBuilderName() const override
	{
		return TEXT("UObject");
	}

	virtual FString GetAssetUniqueName() const override
	{
		return AssetName;
	}

	virtual bool IsBuildThreadSafe() const
	{
		return true;
	}

	virtual bool ShouldBuildAsynchronously() const override
	{
		return true;
	}

	virtual int32 GetAssetVersion() override
	{
		return AssetVersion;
	}

	/**
	 * Override and make a custom serialization function to save/load the important UObject data to disk
	 */
	virtual void SerializeAsset(FArchive& Ar)
	{
	}

	/**
	 * Estimate the size (in bytes) of the Data saved in SerializeData. This makes the buffer memory usage is more efficient.
	 * This is not necessary. However, the closer the estimate is to the actual size the more efficient the memory allocations will be.
	 */
	virtual int64 GetSerializedDataSizeEstimate()
	{
		return 1024;
	}

	FVoidPtrParam Build() override;

	/**
	 * Call this to get the asset named AssetName from the runtime asset cache.
	 * If the asset does not exist on disk, then OnAssetCacheMiss will be called.
	 * Implement OnAssetCacheMiss in order to create the asset that you want cached.
	 */
	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	void GetFromCacheAsync(const FOnAssetCacheComplete& OnComplete);

	UFUNCTION()
	void GetFromCacheAsyncComplete(int32 Handle, FVoidPtrParam DataPtr);

	/**
	 * Make sure Asset is set up and ready to be loaded into
	 */
	virtual void OnAssetPreLoad() {}

	/**
	 * Perform any specific init functions after load
	 */
	virtual void OnAssetPostLoad() {}

	/**
	 * When you get OnAssetCacheMiss you need to load/create the asset that is missing.
	 * Call SaveNewAssetToCache after you're finished creating the asset to save it back into the cache for next time.
	 * This will then trigger OnAssetCacheComplete like normal, so you don't need additional code to handle it.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = RuntimeAssetCache)
	void OnAssetCacheMiss();
	virtual void OnAssetCacheMiss_Implementation()
	{
		//
		// Override and create the new asset here (this is where we would render to a render target, then get the result)
		//

		// Make sure the new asset gets properly cached for next time.
		SaveNewAssetToCache(Asset);
	}

	/**
	 * Call SaveNewAssetToCache to save an asset back into the cache for next time.
	 * This will then trigger OnAssetCacheComplete like normal, so you don't need additional code to handle it.
	 */
	UFUNCTION(BlueprintCallable, Category = RuntimeAssetCache)
	void SaveNewAssetToCache(UObject* NewAsset);

	/**
	 * This merely sets Asset and calls OnSetAsset. Use SaveNewAssetToCache to actually cache a new asset.
	 */
	void SetAsset(UObject* NewAsset);

	/**
	 * Please override in your child class to provide easy access to the Asset.
	 * Declare a specific UObject* variable in your subclass to make it easy for BP nodes to access, e.g. UPROPERTY() UTexture* Texture;
	 * This function will always get called when the Asset changes
	 * So set your specific variable here, e.g.  Texture = Cast<UTexture2D>(NewAsset);
	 */
	virtual void OnSetAsset(UObject* NewAsset)
	{
		// Example:
		// Texture = Cast<UTexture2D>(NewAsset);
	}

	virtual void Cleanup()
	{
		CacheHandle = 0;
		bProcessedCacheMiss = false;
		Asset = nullptr;
	}

public:
	/** The asset version. Changing this will force a new version of the asset to get cached. */
	UPROPERTY(BlueprintReadWrite, Category = RuntimeAssetCache, meta = (ExposeOnSpawn = "true"))
	int32 AssetVersion;

	/** The name of the asset. This should be unique per asset, and is used to look it up from the cache. This should be something that can be known without having Asset in memory (so we can look it up in the cache). */
	UPROPERTY(BlueprintReadWrite, Category = RuntimeAssetCache, meta = (ExposeOnSpawn = "true"))
	FString AssetName;

	int32 CacheHandle;

private:
	UObject* Asset;
	int32 bProcessedCacheMiss : 1;
	FOnRuntimeAssetCacheAsyncComplete GetFromCacheAsyncCompleteDelegate;
	FOnAssetCacheComplete OnAssetCacheComplete;
};

UCLASS(BlueprintType)
class UExampleTextureCacheBuilder : public URuntimeAssetCacheBuilder_ObjectBase
{
	GENERATED_BODY()
public:

	virtual void OnSetAsset(UObject* NewAsset);

	virtual void OnAssetCacheMiss_Implementation() override;

	virtual void SerializeAsset(FArchive& Ar) override;

	virtual void OnAssetPreLoad() override;

	virtual void OnAssetPostLoad() override;

	virtual int64 GetSerializedDataSizeEstimate() override;

	UPROPERTY(BlueprintReadOnly, Category = "Cache")
	TObjectPtr<UTexture2D> Texture;
};
