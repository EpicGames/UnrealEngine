// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Delegates/Delegate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Object.h"
#include "RuntimeAssetCacheInterface.generated.h"

class IRuntimeAssetCacheBuilder;

/** Forward declarations */
class IRuntimeAssetCacheBuilder;

/**
* Useful for passing around void* data and size
*/
USTRUCT(BlueprintType)
struct FVoidPtrParam
{
	GENERATED_BODY()

	FVoidPtrParam(void* InData, int64 InDataSize)
	: Data(InData)
	, DataSize(InDataSize)
	{ }

	FVoidPtrParam()
		: Data(nullptr)
		, DataSize(0)
	{ }

	static FVoidPtrParam NullPtr()
	{
		return FVoidPtrParam();
	}

	inline bool operator!() const
	{
		return (!Data || DataSize <= 0);
	}

	inline explicit operator bool() const
	{
		return (Data && DataSize > 0);
	}

	void* Data;
	int64 DataSize;
};


/** Delegates. */
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnRuntimeAssetCacheAsyncComplete, int32, Handle, FVoidPtrParam, Data);

/**
* Interface for the Runtime Asset Cache. Cache is split into buckets to cache various assets separately.
* Bucket names and maximum sizes are configure via Engine.ini config file using the following syntax
* [RuntimeAssetCache]
* +BucketConfigs=(Name="<Plugin name>", Size=<Maximum bucket size in bytes>)
* This API is fully thread safe.
**/
class FRuntimeAssetCacheInterface
{
public:
	virtual ~FRuntimeAssetCacheInterface() { }

	/**
	* Synchronously gets value from cache. If value is not found, builds entry using CacheBuilder and updates cache.
	* @param CacheBuilder Builder to produce cache key and in the event of a miss.
	* @return Pointer to retrieved cache entry, nullptr on fail. Fail occurs only when
	* - there's no entry in cache and CacheBuilder is nullptr or
	* - CacheBuilder returned nullptr
	**/
	virtual FVoidPtrParam GetSynchronous(IRuntimeAssetCacheBuilder* CacheBuilder) = 0;

	/**
	* Asynchronously checks the cache. If value is not found, builds entry using CacheBuilder and updates cache.
	* @param CacheBuilder Builder to produce cache key and in the event of a miss, return the data.
	* @param OnCompletionDelegate Delegate to call when cache is ready. Delegate is called on main thread.
	* @return Handle to worker.
	**/
	virtual int32 GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder, const FOnRuntimeAssetCacheAsyncComplete& OnCompletionDelegate) = 0;

	/**
	* Asynchronously checks the cache. If value is not found, builds entry using CacheBuilder and updates cache.
	* @param CacheBuilder Builder to produce cache key and in the event of a miss, return the data.
	* @return Handle to worker.
	**/
	virtual int32 GetAsynchronous(IRuntimeAssetCacheBuilder* CacheBuilder) = 0;

	/**
	* Gets cache size.
	* @param Bucket to check.
	* @return Maximum allowed bucket size.
	**/
	virtual int32 GetCacheSize(FName Bucket) const = 0;

	/**
	* Removes all cache entries.
	* @return true if cache was successfully cleaned.
	*/
	virtual bool ClearCache() = 0;

	/**
	* Removes all cache from given bucket.
	* @param Bucket Bucket to clean.
	* @return true if cache was successfully cleaned.
	*/
	virtual bool ClearCache(FName Bucket) = 0;

	/**
	* Waits until worker finishes execution.
	* @param Handle Worker handle to wait for.
	*/
	virtual void WaitAsynchronousCompletion(int32 Handle) = 0;

	/**
	* Gets asynchronous query results.
	* @param Handle Worker handle to check.
	* @return Pointer to retrieved cache entry, nullptr on fail. Fail occurs only when
	* - there's no entry in cache and CacheBuilder is nullptr or
	* - CacheBuilder returned nullptr
	*/
	virtual FVoidPtrParam GetAsynchronousResults(int32 Handle) = 0;

	/**
	* Checks if worker finished execution.
	* @param Handle Worker handle to check.
	* @return True if execution finished, false otherwise.
	*/
	virtual bool PollAsynchronousCompletion(int32 Handle) = 0;

	/**
	 * Adds a number from the thread safe counter which tracks outstanding async requests. This is used to ensure everything is complete prior to shutdown.
	 * @param Value to add to counter. Can be negative.
	 */
	virtual void AddToAsyncCompletionCounter(int32 Addend) = 0;

	/**
	 * Ticks async thread.
	 */
	virtual void Tick() = 0;
};
