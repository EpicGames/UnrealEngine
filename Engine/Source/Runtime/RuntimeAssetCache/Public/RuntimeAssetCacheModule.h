// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FRuntimeAssetCacheInterface;

/** Forward declarations. */
class FRuntimeAssetCacheInterface;

RUNTIMEASSETCACHE_API FRuntimeAssetCacheInterface& GetRuntimeAssetCache();

/**
* Module for the RuntimeAssetCache.
*/
class FRuntimeAssetCacheModuleInterface : public IModuleInterface
{
public:
	/**
	 * Gets runtime asset cache.
	 * @return Runtime asset cache.
	 */
	virtual FRuntimeAssetCacheInterface& GetRuntimeAssetCache() = 0;
};
