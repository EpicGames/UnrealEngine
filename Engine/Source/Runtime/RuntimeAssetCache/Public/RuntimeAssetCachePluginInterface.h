// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "RuntimeAssetCacheInterface.h"
#include "RuntimeAssetCachePluginInterface.generated.h"

class FArchive;

UINTERFACE()
class URuntimeAssetCacheBuilder : public UInterface
{
	GENERATED_BODY()
};

/**
* Interface for runtime asset cache cache builders.
* This API will be called concurrently.
**/
class RUNTIMEASSETCACHE_API IRuntimeAssetCacheBuilder
{
	GENERATED_BODY()

public:

	/**
	* Get the builder type name. This is used to categorize cached data to buckets.
	* @return Type name of builder.
	*/
	virtual const TCHAR* GetBucketConfigName() const = 0;

	/**
	* Get the name of cache builder, this is used as part of the cache key.
	* @return Name of cache builder
	*/
	virtual const TCHAR* GetBuilderName() const = 0;

	/**
	* Returns the name uniquely describing the asset.
	* @return Asset unique name.
	*/
	virtual FString GetAssetUniqueName() const = 0;

	/**
	* Does the work of creating serialized cache entry.
	* @return Pointer to contiguous memory block with serialized cache entry on success, nullptr otherwise.
	*/
	virtual FVoidPtrParam Build() = 0;

	/**
	* Serializes data into archive
	* @param Ar Archive to serialize to.
	* @param InData Data to serialize.
	*/
	virtual void SerializeData(FArchive& Ar, FVoidPtrParam InData)
	{
		Ar.Serialize(InData.Data, InData.DataSize);
	}

	/**
	* Gets asset version to rebuild cache if cached asset is too old.
	* @return Asset version.
	*/
	virtual int32 GetAssetVersion() = 0;

	/**
	* Check if the cached asset is up to date. If the current AssetVersion is greater than the cached AssetVersion, the cached asset will be re-built.
	* @return Whether the cached asset is good (return true) or needs to be rebuilt (return false).
	*/
	virtual bool CachedAssetVersionIsUpToDate(int32 CachedAssetVersion)
	{
		return CachedAssetVersion >= GetAssetVersion();
	}

	/**
	* Gets whether Build function must be called asynchronously.
	* @return true if Build should be called asynchronously, false otherwise.
	*/
	virtual bool ShouldBuildAsynchronously() const
	{
		return false;
	}

	/**
	* Gets whether Build function is threadsafe
	* @return true if Build is threadsafe, false otherwise.
	*/
	virtual bool IsBuildThreadSafe() const = 0;
};
