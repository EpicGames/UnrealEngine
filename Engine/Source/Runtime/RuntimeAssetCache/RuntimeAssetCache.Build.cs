// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class RuntimeAssetCache : ModuleRules
{
	public RuntimeAssetCache(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
			});
	}
}
